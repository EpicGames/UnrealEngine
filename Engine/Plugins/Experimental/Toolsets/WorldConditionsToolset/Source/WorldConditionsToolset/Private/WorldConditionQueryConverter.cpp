// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionQueryConverter.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "WorldConditionQuery.h"

namespace UE::WorldConditionsToolset
{
	FString FWorldConditionQueryConverter::GetName() const
	{
		return TEXT("WorldConditionQueryConverter");
	}

	bool FWorldConditionQueryConverter::CanConvertProperty(TNotNull<const FProperty*> Property)
	{
		const FStructProperty* const StructProp = CastField<FStructProperty>(static_cast<const FProperty*>(Property));
		return StructProp != nullptr
			&& StructProp->Struct == FWorldConditionQueryDefinition::StaticStruct();
	}

	TSharedPtr<FJsonObject> FWorldConditionQueryConverter::PropertyToJsonSchema(TNotNull<const FProperty*> Property)
	{
		TSharedPtr<FJsonObject> Schema = ToolsetStructToJsonSchema(FWorldConditionQueryDefinition::StaticStruct());
		if (Schema.IsValid())
		{
			Schema->SetStringField(TEXT("title"), TEXT("WorldConditionQueryDefinition"));
		}
		return Schema;
	}

	TSharedPtr<FJsonValue> FWorldConditionQueryConverter::PropertyToDefault(
		TNotNull<const FProperty*> Property, const FString& DefaultString)
	{
		return MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
	}

	TSharedPtr<FJsonValue> FWorldConditionQueryConverter::PropertyToJsonData(
		TNotNull<FProperty*> Property, const void* Value)
	{
		const FWorldConditionQueryDefinition* const QueryDef =
			static_cast<const FWorldConditionQueryDefinition*>(Value);
		if (QueryDef == nullptr)
		{
			return MakeShared<FJsonValueNull>();
		}

		// Serialize all UPROPERTYs via the ToolsetJson pipeline.
		// FInstancedStruct conditions get _structType automatically from JsonObjectConverter.
		TSharedPtr<FJsonObject> Result = ToolsetStructToJsonData(
			FWorldConditionQueryDefinition::StaticStruct(), QueryDef);
		if (!Result.IsValid())
		{
			return MakeShared<FJsonValueNull>();
		}

		Result->SetStringField(TEXT("description"), QueryDef->GetDescription().ToString());
		return MakeShared<FJsonValueObject>(Result);
	}

	bool FWorldConditionQueryConverter::JsonDataToProperty(
		const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer)
	{
		if (!JsonValue.IsValid() || JsonValue->Type != EJson::Object)
		{
			return false;
		}

		FWorldConditionQueryDefinition* const QueryDef =
			static_cast<FWorldConditionQueryDefinition*>(OutValue);
		if (QueryDef == nullptr)
		{
			return false;
		}

		// Deserialize all UPROPERTYs via the ToolsetJson pipeline.
		// FInstancedStruct conditions are initialized from _structType automatically by JsonObjectConverter.
		const TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();
		if (!ToolsetJsonDataToStruct(
				JsonObject.ToSharedRef(), FWorldConditionQueryDefinition::StaticStruct(), QueryDef))
		{
			return false;
		}

		// Rebuild SharedDefinition from EditableConditions.
		QueryDef->Initialize(nullptr);
		return true;
	}
}
