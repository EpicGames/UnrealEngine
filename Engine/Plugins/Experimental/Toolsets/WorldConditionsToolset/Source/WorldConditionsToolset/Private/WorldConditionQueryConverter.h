// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetJsonConverter.h"

namespace UE::WorldConditionsToolset
{
	/**
	 * JSON converter for FWorldConditionQueryDefinition.
	 * Serializes the query's UPROPERTYs via the ToolsetJson pipeline,
	 * plus a description convenience field.
	 * On deserialization, rebuilds the SharedDefinition via Initialize().
	 */
	class FWorldConditionQueryConverter : public UE::ToolsetRegistry::FToolsetJsonConverter
	{
	public:
		virtual FString GetName() const override;
		virtual bool CanConvertProperty(TNotNull<const FProperty*> Property) override;
		virtual TSharedPtr<FJsonObject> PropertyToJsonSchema(TNotNull<const FProperty*> Property) override;
		virtual TSharedPtr<FJsonValue> PropertyToDefault(TNotNull<const FProperty*> Property, const FString& DefaultString) override;
		virtual TSharedPtr<FJsonValue> PropertyToJsonData(TNotNull<FProperty*> Property, const void* Value) override;
		virtual bool JsonDataToProperty(const TSharedPtr<FJsonValue>& JsonValue, TNotNull<FProperty*> Property, void* OutValue, UObject* Outer) override;
	};
}
