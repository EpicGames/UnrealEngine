// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionTools.h"

#include "StructUtils/InstancedStruct.h"
#include "WorldConditionBase.h"
#include "WorldConditionQuery.h"

FText UWorldConditionTools::GetQueryDescription(const FWorldConditionQueryDefinition& QueryDefinition)
{
	return QueryDefinition.GetDescription();
}

FText UWorldConditionTools::GetConditionDescription(const FInstancedStruct& Condition)
{
	if (const FWorldConditionBase* const WorldCondition = Condition.GetPtr<FWorldConditionBase>())
	{
		return WorldCondition->GetDescription();
	}
	return FText::GetEmpty();
}
