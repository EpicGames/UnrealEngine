// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"
#include "WorldConditionTools.generated.h"

struct FInstancedStruct;
struct FWorldConditionQueryDefinition;

/** Inspect WorldCondition (FWorldConditionQueryDefinition, FWorldConditionBase) structs. */
UCLASS(Blueprintable)
class UWorldConditionTools : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns a human-readable description of a world condition query.
	 * @param QueryDefinition The query definition to describe.
	 * @return Text description of all conditions in the query.
	 */
	UFUNCTION(meta = (AICallable), Category = "WorldConditions")
	static FText GetQueryDescription(const FWorldConditionQueryDefinition& QueryDefinition);

	/**
	 * Returns a human-readable description of a single world condition.
	 * The condition must be passed as an FInstancedStruct containing an FWorldConditionBase-derived struct.
	 * @param Condition The instanced struct holding the world condition.
	 * @return Text description of the condition, or empty if invalid.
	 */
	UFUNCTION(meta = (AICallable), Category = "WorldConditions")
	static FText GetConditionDescription(const FInstancedStruct& Condition);
};
