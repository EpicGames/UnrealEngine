// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionsToolset.h"

#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "WorldConditionQueryConverter.h"
#include "WorldConditionTools.h"

/*virtual*/ void FWorldConditionsToolsetModule::StartupModule()
{
	WorldConditionQueryConverter = MakeShared<UE::WorldConditionsToolset::FWorldConditionQueryConverter>();
	Register();
}

/*virtual*/ void FWorldConditionsToolsetModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UToolsetRegistry::UnregisterToolsetClass(UWorldConditionTools::StaticClass());

		if (GEditor != nullptr && WorldConditionQueryConverter.IsValid())
		{
			UToolsetRegistrySubsystem* const ToolsetRegistrySubsystem =
				GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>();
			if (ToolsetRegistrySubsystem != nullptr)
			{
				ToolsetRegistrySubsystem->ToolsetRegistry.UnregisterConverter(WorldConditionQueryConverter);
			}
		}
	}

	WorldConditionQueryConverter.Reset();
}

void FWorldConditionsToolsetModule::Register()
{
	if (GEditor == nullptr || !WorldConditionQueryConverter.IsValid())
	{
		return;
	}

	UToolsetRegistry::RegisterToolsetClass(UWorldConditionTools::StaticClass());

	UToolsetRegistrySubsystem* const ToolsetRegistrySubsystem =
		GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>();
	if (ToolsetRegistrySubsystem != nullptr)
	{
		ToolsetRegistrySubsystem->ToolsetRegistry.RegisterConverter(WorldConditionQueryConverter);
	}
}

IMPLEMENT_MODULE(FWorldConditionsToolsetModule, WorldConditionsToolset);
DEFINE_LOG_CATEGORY(LogWorldConditionsToolset);
