// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

namespace UE::ToolsetRegistry { class FToolsetJsonConverter; }

DECLARE_LOG_CATEGORY_EXTERN(LogWorldConditionsToolset, Log, Log);

class FWorldConditionsToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void Register();

	TSharedPtr<UE::ToolsetRegistry::FToolsetJsonConverter> WorldConditionQueryConverter;
};
