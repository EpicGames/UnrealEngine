// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WorldConditionsToolset : ModuleRules
{
	public WorldConditionsToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		PublicDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Engine",
			"Json",
			"ToolsetRegistry",
			"UnrealEd",
			"WorldConditions",
		});
	}
}
